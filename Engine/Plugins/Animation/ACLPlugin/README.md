# ACL Plugin

This plugin integrates the [Animation Compression Library](https://github.com/nfrechette/acl) within Unreal Engine 4 and 5. It is suitable for all your animations and features a low memory footprint, high accuracy, and very fast compression and decompression.

See [here](https://github.com/nfrechette/acl-ue4-plugin) for details!

Note: the plugin releases on GitHub and on the Unreal Marketplace remove a few development only dependencies not required at runtime or within the editor.

## Epic Integration

UE has integrated the ACL Plugin and forked it. Support requests regarding the ACL Plugin should be directed to the Unreal Developer Network. Going forward, Epic will continue the support and development of the plugin.
To clarify which version comes from where, even major versions come from GitHub (e.g. `2.0.4`) while odd major versions come from Epic (e.g. `3.0.4`). Each version forked from Github has 1 added to its major version.
