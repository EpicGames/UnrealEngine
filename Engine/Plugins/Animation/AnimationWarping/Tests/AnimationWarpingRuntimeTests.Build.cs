// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AnimationWarpingRuntimeTests : TestModuleRules
{
	static AnimationWarpingRuntimeTests()
	{
		if (InTestMode)
		{
			TestMetadata = new Metadata();
			TestMetadata.TestName = "AnimationWarpingRuntime";
			TestMetadata.TestShortName = "Animation Warping Runtime";
		}
	}

	public AnimationWarpingRuntimeTests(ReadOnlyTargetRules Target) : base(Target, true)
	{
		bAllowUETypesInNamespaces = true;

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"CoreUObject",
			"Engine",
			"AnimationCore",
			"AnimGraphRuntime",
			"AnimationWarpingRuntime",
		});
	}
}
