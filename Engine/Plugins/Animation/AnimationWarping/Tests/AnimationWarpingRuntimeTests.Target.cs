// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.All)]
public class AnimationWarpingRuntimeTestsTarget : TestTargetRules
{
	public AnimationWarpingRuntimeTestsTarget(TargetInfo Target) : base(Target)
	{
		bUsePlatformFileStub = true;
		bMockEngineDefaults = true;

		bCompileAgainstEngine = true;
		bCompileAgainstApplicationCore = true;

		bTestsRequireEngine = true;
	}
}
