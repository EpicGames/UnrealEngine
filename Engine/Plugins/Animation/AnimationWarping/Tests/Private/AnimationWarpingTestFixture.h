// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/Skeleton.h"
#include "BoneContainer.h"
#include "BoneControllers/AnimNode_OrientationWarping.h"
#include "BoneControllers/AnimNode_StrideWarping.h"
#include "BonePose.h"
#include "Misc/MemStack.h"
#include "ReferenceSkeleton.h"
#include "UObject/Package.h"

#include "TestHarness.h"

namespace UE::AnimationWarping::Tests
{
	struct FTestBone
	{
		FName Name;
		FName ParentName;
	};

	inline const FTestBone GTestBones[] =
	{
		{ TEXT("root"),         NAME_None },
		{ TEXT("pelvis"),        TEXT("root") },
		{ TEXT("thigh_l"),       TEXT("pelvis") },
		{ TEXT("calf_l"),        TEXT("thigh_l") },
		{ TEXT("foot_l"),        TEXT("calf_l") },
		{ TEXT("ball_l"),        TEXT("foot_l") },
		{ TEXT("thigh_r"),       TEXT("pelvis") },
		{ TEXT("calf_r"),        TEXT("thigh_r") },
		{ TEXT("foot_r"),        TEXT("calf_r") },
		{ TEXT("ball_r"),        TEXT("foot_r") },
		{ TEXT("ik_foot_root"),  TEXT("root") },
		{ TEXT("ik_foot_l"),     TEXT("ik_foot_root") },
		{ TEXT("ik_foot_r"),     TEXT("ik_foot_root") },
	};

	// Build a transient biped skeleton matching GTestBones. Owned by the transient package.
	inline USkeleton* MakeTestSkeleton()
	{
		USkeleton* Skeleton = NewObject<USkeleton>(GetTransientPackage(), NAME_None, RF_Transient);

		FReferenceSkeletonModifier Modifier(Skeleton);
		for (const FTestBone& Bone : GTestBones)
		{
			const int32 ParentIndex = (Bone.ParentName == NAME_None)
				? INDEX_NONE
				: Modifier.FindBoneIndex(Bone.ParentName);
			const FMeshBoneInfo Info(Bone.Name, Bone.Name.ToString(), ParentIndex);
			Modifier.Add(Info, FTransform::Identity);
		}

		return Skeleton;
	}

	// Widen access to the protected EvaluateSkeletalControl_AnyThread and UpdateInternal for tests only.
	template <typename TNode>
	struct TTestAdapter : public TNode
	{
		using TNode::EvaluateSkeletalControl_AnyThread;
		using TNode::UpdateInternal;
	};

	template <typename TNode>
	inline void InitializeNode(TTestAdapter<TNode>& Node, FAnimInstanceProxy& Proxy)
	{
		FAnimationInitializeContext InitContext(&Proxy);
		Node.Initialize_AnyThread(InitContext);

		FAnimationCacheBonesContext CacheContext(&Proxy);
		Node.CacheBones_AnyThread(CacheContext);
	}

	// Drive UpdateInternal directly with a specific DeltaTime. We skip the base
	// Update_AnyThread because it calls FAnimInstanceProxy::GetSkeleton(), which
	// ensureAlways's on a skeleton being set — something only the full anim-instance
	// bootstrap wires up. UpdateInternal alone handles the per-node state we need
	// (e.g. StrideWarping's CachedDeltaTime).
	template <typename TNode>
	inline void UpdateWithDeltaTime(TTestAdapter<TNode>& Node, FAnimInstanceProxy& Proxy, float DeltaTime)
	{
		FAnimationUpdateContext Update(&Proxy, DeltaTime);
		Node.UpdateInternal(Update);
	}

	template <typename TNode>
	inline void EvaluateNode(TTestAdapter<TNode>& Node, FAnimInstanceProxy& Proxy, FComponentSpacePoseContext& Context, TArray<FBoneTransform>& OutBoneTransforms)
	{
		InitializeNode(Node, Proxy);
		OutBoneTransforms.Reset();
		Node.EvaluateSkeletalControl_AnyThread(Context, OutBoneTransforms);
	}

	inline void ConfigureStrideWarping(FAnimNode_StrideWarping& Node)
	{
		Node.PelvisBone.BoneName = TEXT("pelvis");
		Node.IKFootRootBone.BoneName = TEXT("ik_foot_root");
		Node.Mode = EWarpingEvaluationMode::Manual;
		Node.bOrientStrideDirectionUsingFloorNormal = true;
		Node.bCompensateIKUsingFKThighRotation = true;
		Node.bClampIKUsingFKLimits = true;
		Node.FootDefinitions.Reset();
		{
			FStrideWarpingFootDefinition Def;
			Def.IKFootBone.BoneName = TEXT("ik_foot_l");
			Def.FKFootBone.BoneName = TEXT("foot_l");
			Def.ThighBone.BoneName = TEXT("thigh_l");
			Node.FootDefinitions.Add(Def);
		}
		{
			FStrideWarpingFootDefinition Def;
			Def.IKFootBone.BoneName = TEXT("ik_foot_r");
			Def.FKFootBone.BoneName = TEXT("foot_r");
			Def.ThighBone.BoneName = TEXT("thigh_r");
			Node.FootDefinitions.Add(Def);
		}
	}

	inline void ConfigureOrientationWarping(FAnimNode_OrientationWarping& Node)
	{
		Node.Mode = EWarpingEvaluationMode::Manual;
		Node.IKFootRootBone.BoneName = TEXT("ik_foot_root");
		Node.IKFootBones.Reset();
		Node.IKFootBones.Add(FBoneReference(TEXT("ik_foot_l")));
		Node.IKFootBones.Add(FBoneReference(TEXT("ik_foot_r")));
		Node.OrientationAngle = 15.f;
	}

	inline float MakeNaNFloat() { return FMath::Sqrt(-1.f); }
	inline FVector MakeNaNVector() { return FVector(MakeNaNFloat(), 0.0, 0.0); }

	// Overwrite a pose bone's component-space translation. FCSPose::SetComponentSpaceTransform
	// does not assert on translation values other than NaN, so this is safe for extreme/zero
	// values and test-controlled degenerate geometry.
	inline void SetBoneLocation(FComponentSpacePoseContext& Context, const USkeleton& Skeleton, FName BoneName, const FVector& Location)
	{
		const int32 MeshIndex = Skeleton.GetReferenceSkeleton().FindBoneIndex(BoneName);
		if (MeshIndex == INDEX_NONE)
		{
			return;
		}
		const FCompactPoseBoneIndex CompactIndex = Context.Pose.GetPose().GetBoneContainer().GetCompactPoseIndexFromSkeletonIndex(MeshIndex);
		if (CompactIndex.GetInt() == INDEX_NONE)
		{
			return;
		}
		FTransform BoneTransform = Context.Pose.GetComponentSpaceTransform(CompactIndex);
		BoneTransform.SetLocation(Location);
		Context.Pose.SetComponentSpaceTransform(CompactIndex, BoneTransform);
	}

	inline bool AnyOutputContainsNaN(const TArray<FBoneTransform>& OutBoneTransforms)
	{
		for (const FBoneTransform& BT : OutBoneTransforms)
		{
			if (BT.Transform.ContainsNaN())
			{
				return true;
			}
		}
		return false;
	}

	// Catch2 fixture. Builds a transient skeleton, initialises a synthetic FAnimInstanceProxy
	// against its ref skeleton, and sets up a component-space pose context at the ref pose.
	// Each TEST_CASE_METHOD gets a fresh fixture.
	struct FAnimWarpingFixture
	{
		FMemMark MemMark;
		USkeleton* Skeleton = nullptr;
		FAnimInstanceProxy Proxy;
		FComponentSpacePoseContext Context;
		TArray<FBoneIndexType> RequiredBones;

		FAnimWarpingFixture()
			: MemMark(FMemStack::Get())
			, Context(&Proxy)
		{
			Skeleton = MakeTestSkeleton();
			REQUIRE(Skeleton != nullptr);

			const int32 NumBones = Skeleton->GetReferenceSkeleton().GetNum();
			RequiredBones.Reset(NumBones);
			for (int32 Idx = 0; Idx < NumBones; ++Idx)
			{
				RequiredBones.Add(static_cast<FBoneIndexType>(Idx));
			}

			const UE::Anim::FCurveFilterSettings FilterSettings(UE::Anim::ECurveFilterMode::DisallowAll);
			Proxy.GetRequiredBones().InitializeTo(RequiredBones, FilterSettings, *Skeleton);

			Context.ResetToRefPose();
		}
	};
}
