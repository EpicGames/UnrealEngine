// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationWarpingTestFixture.h"

// Coverage note:
//   FCSPose<FCompactPose>::GetComponentSpaceTransform asserts !ContainsNaN on read
//   (BonePose.h) in every non-shipping build. The ensureMsgf guards added to the
//   warping nodes catch NaN in paths where that framework assert is compiled out
//   (Shipping/Test) or does not otherwise fire (e.g. when a NaN is produced by
//   later arithmetic inside the node). These LLT tests run against a Development
//   Engine build, so they cannot synthesise a NaN-poisoned pose and still reach
//   our node-level guards. They instead exercise the reachable invalid-input
//   surface: NaN node properties, degenerate bone geometry, extreme values,
//   multi-tick stability, empty configurations, and zero-DeltaTime updates.

namespace UE::AnimationWarping::Tests
{

// ---------------------------------------------------------------------------
// StrideWarping — the node that owns UE-363190.
// ---------------------------------------------------------------------------

TEST_CASE_METHOD(FAnimWarpingFixture,
	"Animation::Warping::StrideWarping::CleanInput",
	"[AnimationWarping][StrideWarping]")
{
	TTestAdapter<FAnimNode_StrideWarping> Node;
	ConfigureStrideWarping(Node);

	TArray<FBoneTransform> OutBoneTransforms;
	EvaluateNode(Node, Proxy, Context, OutBoneTransforms);

	CHECK_FALSE(AnyOutputContainsNaN(OutBoneTransforms));
}

TEST_CASE_METHOD(FAnimWarpingFixture,
	"Animation::Warping::StrideWarping::NaNStrideDirection",
	"[AnimationWarping][StrideWarping]")
{
	TTestAdapter<FAnimNode_StrideWarping> Node;
	ConfigureStrideWarping(Node);
	Node.StrideDirection = MakeNaNVector();

	TArray<FBoneTransform> OutBoneTransforms;
	EvaluateNode(Node, Proxy, Context, OutBoneTransforms);

	// Manual mode early-returns via UE_LOGF; nothing emitted.
	CHECK(OutBoneTransforms.Num() == 0);
}

TEST_CASE_METHOD(FAnimWarpingFixture,
	"Animation::Warping::StrideWarping::NaNStrideScale",
	"[AnimationWarping][StrideWarping]")
{
	TTestAdapter<FAnimNode_StrideWarping> Node;
	ConfigureStrideWarping(Node);
	Node.StrideScale = MakeNaNFloat();

	TArray<FBoneTransform> OutBoneTransforms;
	EvaluateNode(Node, Proxy, Context, OutBoneTransforms);

	CHECK(OutBoneTransforms.Num() == 0);
}

TEST_CASE_METHOD(FAnimWarpingFixture,
	"Animation::Warping::StrideWarping::NaNFloorNormalProperty",
	"[AnimationWarping][StrideWarping]")
{
	TTestAdapter<FAnimNode_StrideWarping> Node;
	ConfigureStrideWarping(Node);
	Node.FloorNormalDirection.Mode = EWarpingVectorMode::ComponentSpaceVector;
	Node.FloorNormalDirection.Value = MakeNaNVector();

	TArray<FBoneTransform> OutBoneTransforms;
	EvaluateNode(Node, Proxy, Context, OutBoneTransforms);

	// Node logs an error and falls back to UpVector; output must still be clean.
	CHECK_FALSE(AnyOutputContainsNaN(OutBoneTransforms));
}

TEST_CASE_METHOD(FAnimWarpingFixture,
	"Animation::Warping::StrideWarping::NaNGravityDirectionProperty",
	"[AnimationWarping][StrideWarping]")
{
	TTestAdapter<FAnimNode_StrideWarping> Node;
	ConfigureStrideWarping(Node);
	Node.GravityDirection.Mode = EWarpingVectorMode::ComponentSpaceVector;
	Node.GravityDirection.Value = MakeNaNVector();

	TArray<FBoneTransform> OutBoneTransforms;
	EvaluateNode(Node, Proxy, Context, OutBoneTransforms);

	CHECK_FALSE(AnyOutputContainsNaN(OutBoneTransforms));
}

TEST_CASE_METHOD(FAnimWarpingFixture,
	"Animation::Warping::StrideWarping::CollocatedThighAndFoot",
	"[AnimationWarping][StrideWarping]")
{
	// Force the left leg into a degenerate configuration: thigh, FK foot and IK foot
	// all at the same component-space location. TargetDir becomes zero via GetSafeNormal,
	// exercising FQuat::FindBetweenNormals's zero-input branch and the new
	// check(!ClampedFootLocation.ContainsNaN()) guard in the clamp path.
	const FVector HipLoc(0, 0, 100);
	SetBoneLocation(Context, *Skeleton, TEXT("thigh_l"), HipLoc);
	SetBoneLocation(Context, *Skeleton, TEXT("foot_l"), HipLoc);
	SetBoneLocation(Context, *Skeleton, TEXT("ik_foot_l"), HipLoc);

	TTestAdapter<FAnimNode_StrideWarping> Node;
	ConfigureStrideWarping(Node);

	TArray<FBoneTransform> OutBoneTransforms;
	EvaluateNode(Node, Proxy, Context, OutBoneTransforms);

	CHECK_FALSE(AnyOutputContainsNaN(OutBoneTransforms));
}

TEST_CASE_METHOD(FAnimWarpingFixture,
	"Animation::Warping::StrideWarping::ExtremeBonePositions",
	"[AnimationWarping][StrideWarping]")
{
	// Large-world setup: FKLength / IKLength must stay finite and the new
	// check(IsFinite(...)) guards in the clamp path must not spuriously fire.
	const FVector HipLoc(1.0e6, 0, 0);
	const FVector FootLoc(1.0e6, 0, -100);
	const FVector IKFootLoc(1.0e6, 0, -110);
	SetBoneLocation(Context, *Skeleton, TEXT("thigh_l"), HipLoc);
	SetBoneLocation(Context, *Skeleton, TEXT("foot_l"), FootLoc);
	SetBoneLocation(Context, *Skeleton, TEXT("ik_foot_l"), IKFootLoc);

	TTestAdapter<FAnimNode_StrideWarping> Node;
	ConfigureStrideWarping(Node);

	TArray<FBoneTransform> OutBoneTransforms;
	EvaluateNode(Node, Proxy, Context, OutBoneTransforms);

	CHECK_FALSE(AnyOutputContainsNaN(OutBoneTransforms));
}

TEST_CASE_METHOD(FAnimWarpingFixture,
	"Animation::Warping::StrideWarping::MultiTickStable",
	"[AnimationWarping][StrideWarping]")
{
	TTestAdapter<FAnimNode_StrideWarping> Node;
	ConfigureStrideWarping(Node);
	InitializeNode(Node, Proxy);

	const float Scales[] = { 1.0f, 0.5f, 1.25f, 2.0f, 0.75f };
	for (float Scale : Scales)
	{
		Node.StrideScale = Scale;
		UpdateWithDeltaTime(Node, Proxy, 1.f / 60.f);

		Context.ResetToRefPose();
		TArray<FBoneTransform> OutBoneTransforms;
		Node.EvaluateSkeletalControl_AnyThread(Context, OutBoneTransforms);

		CHECK_FALSE(AnyOutputContainsNaN(OutBoneTransforms));
	}
}

TEST_CASE_METHOD(FAnimWarpingFixture,
	"Animation::Warping::StrideWarping::EmptyFootDefinitions",
	"[AnimationWarping][StrideWarping]")
{
	TTestAdapter<FAnimNode_StrideWarping> Node;
	ConfigureStrideWarping(Node);
	Node.FootDefinitions.Reset();

	InitializeNode(Node, Proxy);
	CHECK_FALSE(Node.IsValidToEvaluate(nullptr, Context.Pose.GetPose().GetBoneContainer()));
}

TEST_CASE_METHOD(FAnimWarpingFixture,
	"Animation::Warping::StrideWarping::ZeroDeltaTime",
	"[AnimationWarping][StrideWarping]")
{
	TTestAdapter<FAnimNode_StrideWarping> Node;
	ConfigureStrideWarping(Node);
	InitializeNode(Node, Proxy);
	UpdateWithDeltaTime(Node, Proxy, 0.f);

	TArray<FBoneTransform> OutBoneTransforms;
	Node.EvaluateSkeletalControl_AnyThread(Context, OutBoneTransforms);

	CHECK_FALSE(AnyOutputContainsNaN(OutBoneTransforms));
}

// SlopeWarping tests intentionally omitted. The node's Initialize_AnyThread
// calls FAnimInstanceProxy::GetSkelMeshComponent() which ensureAlways's on the
// proxy having a real SkeletalMeshComponent wired up. The LLT fixture has none,
// and the LowLevelTestsRunner forwards every handled ensure to Catch2 as an
// explicit failure — so any test that touches SlopeWarping's Initialize path
// would auto-fail before the test body runs.

// ---------------------------------------------------------------------------
// OrientationWarping — writes back in place via SetComponentSpaceTransform;
// OutBoneTransforms may remain empty. Reaching the end of evaluation proves
// no fatal assert fired.
// ---------------------------------------------------------------------------

TEST_CASE_METHOD(FAnimWarpingFixture,
	"Animation::Warping::OrientationWarping::CleanInput",
	"[AnimationWarping][OrientationWarping]")
{
	TTestAdapter<FAnimNode_OrientationWarping> Node;
	ConfigureOrientationWarping(Node);

	TArray<FBoneTransform> OutBoneTransforms;
	EvaluateNode(Node, Proxy, Context, OutBoneTransforms);

	SUCCEED("OrientationWarping completed without crashing");
}

TEST_CASE_METHOD(FAnimWarpingFixture,
	"Animation::Warping::OrientationWarping::ExtremeAngle",
	"[AnimationWarping][OrientationWarping]")
{
	TTestAdapter<FAnimNode_OrientationWarping> Node;
	ConfigureOrientationWarping(Node);
	Node.OrientationAngle = 10000.f; // far outside [-180, 180]

	TArray<FBoneTransform> OutBoneTransforms;
	EvaluateNode(Node, Proxy, Context, OutBoneTransforms);

	SUCCEED("Extreme angle evaluated without crashing");
}

TEST_CASE_METHOD(FAnimWarpingFixture,
	"Animation::Warping::OrientationWarping::ZeroDeltaTime",
	"[AnimationWarping][OrientationWarping]")
{
	TTestAdapter<FAnimNode_OrientationWarping> Node;
	ConfigureOrientationWarping(Node);
	InitializeNode(Node, Proxy);
	// OrientationWarping::UpdateInternal calls Context.GetMessage<T>() which ensures on
	// SharedContext. The node does not cache DeltaTime, so evaluate directly.
	TArray<FBoneTransform> OutBoneTransforms;
	Node.EvaluateSkeletalControl_AnyThread(Context, OutBoneTransforms);

	SUCCEED("Zero-DeltaTime evaluated without crashing");
}

TEST_CASE_METHOD(FAnimWarpingFixture,
	"Animation::Warping::OrientationWarping::NaNOrientationAngle",
	"[AnimationWarping][OrientationWarping]")
{
	// Simulate a user authoring NaN into the OrientationAngle pin. The node should log
	// an error and fall back to 0 — no NaN should land in the output pose.
	TTestAdapter<FAnimNode_OrientationWarping> Node;
	ConfigureOrientationWarping(Node);
	InitializeNode(Node, Proxy);
	// ActualOrientationAngleRad is set from OrientationAngle internally during Reset/Update;
	// poking the runtime directly is the most reliable way to exercise the guard from LLT.
	Node.OrientationAngle = MakeNaNFloat();

	TArray<FBoneTransform> OutBoneTransforms;
	Node.EvaluateSkeletalControl_AnyThread(Context, OutBoneTransforms);

	// OrientationWarping writes in-place via SetComponentSpaceTransform. Sample the root
	// bone after evaluation to verify we didn't write NaN into the pose.
	const FCompactPoseBoneIndex RootIndex(0);
	const FTransform RootAfter = Context.Pose.GetComponentSpaceTransform(RootIndex);
	CHECK_FALSE(RootAfter.ContainsNaN());
}

TEST_CASE_METHOD(FAnimWarpingFixture,
	"Animation::Warping::OrientationWarping::NaNDistributedBoneOrientationAlpha",
	"[AnimationWarping][OrientationWarping]")
{
	TTestAdapter<FAnimNode_OrientationWarping> Node;
	ConfigureOrientationWarping(Node);
	Node.DistributedBoneOrientationAlpha = MakeNaNFloat();
	InitializeNode(Node, Proxy);

	TArray<FBoneTransform> OutBoneTransforms;
	Node.EvaluateSkeletalControl_AnyThread(Context, OutBoneTransforms);

	const FCompactPoseBoneIndex RootIndex(0);
	const FTransform RootAfter = Context.Pose.GetComponentSpaceTransform(RootIndex);
	CHECK_FALSE(RootAfter.ContainsNaN());
}

} // namespace UE::AnimationWarping::Tests
