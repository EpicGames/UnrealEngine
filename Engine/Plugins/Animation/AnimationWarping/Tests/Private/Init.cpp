// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestCommon/Initialization.h"

#include "TestHarness.h"

// Engine bootstrap for this LLT: InitAll brings up CoreUObject + Engine so that
// NewObject<USkeleton>(GetTransientPackage(), ...) and FAnimInstanceProxy can
// construct. Matches the pattern used by PCGTests (Engine/Plugins/PCG/Tests/Private/Init.cpp).
GROUP_BEFORE_GLOBAL(Catch::DefaultGroup)
{
	InitAll(true /*bAllowLogging*/, true /*bMultithreaded*/);
}

GROUP_AFTER_GLOBAL(Catch::DefaultGroup)
{
	CleanupAll();
	GIsRunning = false;
}
